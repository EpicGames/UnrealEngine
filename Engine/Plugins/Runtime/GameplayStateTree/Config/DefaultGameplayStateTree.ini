[CoreRedirects]
+ClassRedirects=(OldName="/Script/StateTreeModule.StateTreeBrainComponent", NewName="/Script/GameplayStateTreeModule.StateTreeComponent")
+ClassRedirects=(OldName="/Script/StateTreeModule.BrainComponentStateTreeSchema", NewName="/Script/GameplayStateTreeModule.StateTreeComponentSchema")
