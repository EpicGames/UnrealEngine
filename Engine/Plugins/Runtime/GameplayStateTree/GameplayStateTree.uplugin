{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "GameplayStateTree",
	"Description" : "StateTree for AI/Gameplay Behaviors",
	"Category" : "Gameplay",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules": [
		{
			"Name": "GameplayStateTreeModule",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "StateTree",
			"Enabled": true
		}
	]
}