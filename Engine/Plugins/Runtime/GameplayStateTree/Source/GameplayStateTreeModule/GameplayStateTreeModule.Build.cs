// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GameplayStateTreeModule : ModuleRules
	{
		public GameplayStateTreeModule(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePaths.AddRange(
				new string[] {
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"AIModule",
					"Core",
					"CoreUObject",
					"Engine",
					"GameplayTags",
					"GameplayTasks",
					"NavigationSystem",
					"PropertyBindingUtils",
					"StateTreeModule",
				}
			);
		}
	}
}
