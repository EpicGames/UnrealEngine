// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/GameplayStateTreeBTUtils.h"

#include "BehaviorTree/BehaviorTreeTypes.h"
#include "StateTreeExecutionTypes.h"

namespace GameplayStateTreeBTUtils
{
	EBTNodeResult::Type StateTreeRunStatusToBTNodeResult(EStateTreeRunStatus Result)
	{
		switch (Result)
		{
		case EStateTreeRunStatus::Succeeded:
			return EBTNodeResult::Succeeded;
		case EStateTreeRunStatus::Running:
			return EBTNodeResult::InProgress;
		case EStateTreeRunStatus::Failed:
		case EStateTreeRunStatus::Stopped:
		default:
			return EBTNodeResult::Failed;
		}
	}
}