// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_RunDynamicStateTree.h"

#include "BehaviorTree/BehaviorTreeComponent.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTree/GameplayStateTreeBTUtils.h"
#include "GameFramework/Actor.h"
#include "Internationalization/Regex.h"
#include "StateTreeExecutionContext.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_RunDynamicStateTree)

bool UBTTask_RunDynamicStateTree::SetDynamicStateTree(UBehaviorTreeComponent& OwnerComp, FGameplayTag InInjectTag, const FStateTreeReference& InStateTree, const FSetContextDataDelegate& InSetContextDataDelegate, float InInterval, float InRandomDeviation, UBTCompositeNode* OptionalStartNode /*= nullptr*/)
{
	bool bInjected = false;
	auto ReplaceDynamicStateTree = [&](UBTTaskNode& TaskNode, const FBehaviorTreeInstance& InstanceInfo, int32 InstanceIndex)
		{
			UBTTask_RunDynamicStateTree* StateTreeTask = Cast<UBTTask_RunDynamicStateTree>(&TaskNode);
			if (StateTreeTask && StateTreeTask->InjectionTag == InInjectTag)
			{
				const uint8* NodeMemory = StateTreeTask->GetNodeMemory<uint8>(InstanceInfo);
				if (UBTTask_RunDynamicStateTree* InstancedNode = Cast<UBTTask_RunDynamicStateTree>(StateTreeTask->GetNodeInstance(OwnerComp, (uint8*)NodeMemory)))
				{
					InstancedNode->SetStateTreeToRun(OwnerComp, InStateTree, InSetContextDataDelegate, InInterval, InRandomDeviation);
					UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Log, TEXT("Replaced state tree in %s with %s (tag: %s)"),
						*UBehaviorTreeTypes::DescribeNodeHelper(StateTreeTask), *GetNameSafe(InStateTree.GetStateTree()), *InInjectTag.ToString());
					bInjected = true;

					if (InstanceInfo.ActiveNodeType == EBTActiveNode::ActiveTask && StateTreeTask == InstanceInfo.ActiveNode)
					{
						UBTCompositeNode* RestartNode = StateTreeTask->GetParentNode();
						int32 RestartChildIdx = RestartNode->GetChildIndex(*StateTreeTask);
						OwnerComp.RequestExecution(RestartNode, InstanceIndex, StateTreeTask, RestartChildIdx, EBTNodeResult::Aborted);
					}
				}
			}
		};

	if (OptionalStartNode)
	{
		OwnerComp.ForEachChildTask(*OptionalStartNode, OwnerComp.FindInstanceContainingNode(OptionalStartNode), ReplaceDynamicStateTree);
	}
	else
	{
		OwnerComp.ForEachChildTask(ReplaceDynamicStateTree);
	}

	return bInjected;
}


UBTTask_RunDynamicStateTree::UBTTask_RunDynamicStateTree(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	INIT_TASK_NODE_NOTIFY_FLAGS();
	NodeName = TEXT("Run Dynamic State Tree");
	bCreateNodeInstance = true;
	bTickIntervals = true;
}

EBTNodeResult::Type UBTTask_RunDynamicStateTree::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	if (!StateTreeRef.IsValid())
	{
		return EBTNodeResult::Failed;
	}

	FStateTreeExecutionContext Context(*OwnerComp.GetOwner(), *StateTreeRef.GetStateTree(), InstanceData);
	SetContextDataDelegate.ExecuteIfBound(Context, OwnerComp, InjectionTag);
	const EStateTreeRunStatus StartStatus = Context.Start(StateTreeRef.GetGlobalParameters());
	return GameplayStateTreeBTUtils::StateTreeRunStatusToBTNodeResult(StartStatus);
}

void UBTTask_RunDynamicStateTree::TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	if (!StateTreeRef.IsValid())
	{
		FinishLatentTask(OwnerComp, EBTNodeResult::Failed);
	}

	EStateTreeRunStatus TickStatus = EStateTreeRunStatus::Unset;
	{
		FStateTreeExecutionContext Context(*OwnerComp.GetOwner(), *StateTreeRef.GetStateTree(), InstanceData);
		SetContextDataDelegate.ExecuteIfBound(Context, OwnerComp, InjectionTag);
		TickStatus = Context.Tick(DeltaSeconds);
	}

	if (TickStatus != EStateTreeRunStatus::Running)
	{
		FinishLatentTask(OwnerComp, GameplayStateTreeBTUtils::StateTreeRunStatusToBTNodeResult(TickStatus));
	}
	else
	{
		SetNextTickTime(NodeMemory, FMath::Max(0.f, Interval + FMath::FRandRange(-RandomDeviation, RandomDeviation)));
	}
}

void UBTTask_RunDynamicStateTree::OnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult)
{
	if (StateTreeRef.IsValid())
	{
		FStateTreeExecutionContext Context(*OwnerComp.GetOwner(), *StateTreeRef.GetStateTree(), InstanceData);
		SetContextDataDelegate.ExecuteIfBound(Context, OwnerComp, InjectionTag);
		Context.Stop();
	}
}

void UBTTask_RunDynamicStateTree::SetStateTreeToRun(UBehaviorTreeComponent& OwnerComp, const FStateTreeReference& InStateTree, const FSetContextDataDelegate& InSetContextDelegate, float InInterval, float InRandomDeviation)
{
	if (StateTreeRef.IsValid())
	{
		FStateTreeExecutionContext Context(*OwnerComp.GetOwner(), *StateTreeRef.GetStateTree(), InstanceData);
		if (Context.GetStateTreeRunStatus() == EStateTreeRunStatus::Running)
		{
			SetContextDataDelegate.ExecuteIfBound(Context, OwnerComp, InjectionTag);
			Context.Stop();
		}
	}

	StateTreeRef = InStateTree;
	SetContextDataDelegate  = InSetContextDelegate;
	Interval = InInterval;
	RandomDeviation = InRandomDeviation;
}

FString UBTTask_RunDynamicStateTree::GetStaticDescription() const
{
	return FString::Printf(TEXT("%s: %s"), *Super::GetStaticDescription(), *InjectionTag.ToString());
}

void UBTTask_RunDynamicStateTree::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	Super::DescribeRuntimeValues(OwnerComp, NodeMemory, Verbosity, Values);
	Values.Add(FString::Printf(TEXT("State tree: %s"), *GetNameSafe(StateTreeRef.GetStateTree())));
	if (StateTreeRef.IsValid())
	{
		Values.Add(FString::Printf(TEXT("State tree path: %s"), *StateTreeRef.GetStateTree()->GetPathName()));
		Values.Add(FString::Printf(TEXT("Interval: %.2f RandomDeviation: %.2f"), Interval, RandomDeviation));
	}
}

#if WITH_EDITOR
const UObject* UBTTask_RunDynamicStateTree::GetAssociatedAsset(TOptional<FBehaviorTreeNodeDebugContext> DebugContext) const
{
	if (DebugContext.IsSet())
	{
		static const FRegexPattern RegexPatern(FString(TEXT("[\\n\\r].*state tree path:\\s*([^\\n\\r]*)")));
		FRegexMatcher Matcher(RegexPatern, DebugContext->RuntimeDescription);
		if (Matcher.FindNext())
		{
			// Capture group 0 is the whole subtree line, capture group 1 is the path.
			const FString StateTreePath = Matcher.GetCaptureGroup(1);
			return FindObject<UStateTree>(nullptr, *StateTreePath);
		}
	}
	return StateTreeRef.GetStateTree();
}
#endif //WITH_EDITOR
