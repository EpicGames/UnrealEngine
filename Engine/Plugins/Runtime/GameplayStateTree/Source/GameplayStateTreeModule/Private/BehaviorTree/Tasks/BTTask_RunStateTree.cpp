// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_RunStateTree.h"

#include "BehaviorTree/GameplayStateTreeBTUtils.h"
#include "Components/StateTreeAIComponentSchema.h"
#include "StateTree.h"
#include "StateTreeExecutionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_RunStateTree)

namespace UE::GameplayStateTree::Private
{
	const FLazyName ActorName = "Actor";
	AActor* GetSchemeActorContext(FStateTreeExecutionContext& Context)
	{
		FStateTreeDataView ActorContextView = Context.GetContextDataByName(ActorName);
		return ActorContextView.IsValid() && ActorContextView.GetStruct()->IsChildOf(AActor::StaticClass()) ? ActorContextView.GetMutablePtr<AActor>() : nullptr;
	}
}

UBTTask_RunStateTree::UBTTask_RunStateTree(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
	: Super(ObjectInitializer)
{
	INIT_TASK_NODE_NOTIFY_FLAGS();
	NodeName = TEXT("Run State Tree");
	bCreateNodeInstance = true;
	bTickIntervals = true;
}

EBTNodeResult::Type UBTTask_RunStateTree::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	SchemaActor.Reset();

	if (StateTreeRef.IsValid())
	{
		FStateTreeExecutionContext Context(*OwnerComp.GetOwner(), *StateTreeRef.GetStateTree(), InstanceData);
		if (SetContextRequirements(OwnerComp, Context))
		{
			const EStateTreeRunStatus StartStatus = Context.Start(StateTreeRef.GetGlobalParameters());
			if (StartStatus == EStateTreeRunStatus::Running)
			{
				SchemaActor = UE::GameplayStateTree::Private::GetSchemeActorContext(Context);
			}
			return GameplayStateTreeBTUtils::StateTreeRunStatusToBTNodeResult(StartStatus);
		}
	}

	return EBTNodeResult::Failed;
}

void UBTTask_RunStateTree::TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	if (StateTreeRef.IsValid())
	{
		FStateTreeExecutionContext Context(*OwnerComp.GetOwner(), *StateTreeRef.GetStateTree(), InstanceData);
		if (SetContextRequirements(OwnerComp, Context))
		{
			ensureMsgf(SchemaActor == UE::GameplayStateTree::Private::GetSchemeActorContext(Context), TEXT("The schema actor is not the same from the previous tick."));
			const EStateTreeRunStatus TickStatus = Context.Tick(DeltaSeconds);
			if (TickStatus != EStateTreeRunStatus::Running)
			{
				FinishLatentTask(OwnerComp, GameplayStateTreeBTUtils::StateTreeRunStatusToBTNodeResult(TickStatus));
			}
			else
			{
				SetNextTickTime(NodeMemory, FMath::Max(0.f, Interval + FMath::FRandRange(-RandomDeviation, RandomDeviation)));
			}
			return;
		}
	}

	FinishLatentTask(OwnerComp, EBTNodeResult::Failed);
}

void UBTTask_RunStateTree::OnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult)
{
	if (StateTreeRef.IsValid())
	{
		FStateTreeExecutionContext Context(*OwnerComp.GetOwner(), *StateTreeRef.GetStateTree(), InstanceData);
		if(!SetContextRequirements(OwnerComp, Context))
		{
			// If the controller is destroyed / GCed before this task, the state tree won't be stopped properly causing issues.
			//@todo This is a temporary fix until the schema can save the data appropriately.
			Context.SetContextDataByName(UE::GameplayStateTree::Private::ActorName, FStateTreeDataView(SchemaActor.Get()));
		}
		if (ensure(Context.AreContextDataViewsValid()))
		{
			Context.Stop();
		}
	}
}

bool UBTTask_RunStateTree::SetContextRequirements(UBehaviorTreeComponent& OwnerComp, FStateTreeExecutionContext& Context)
{
	Context.SetCollectExternalDataCallback(FOnCollectStateTreeExternalData::CreateUObject(this, &UBTTask_RunStateTree::CollectExternalData));
	return UStateTreeAIComponentSchema::SetContextRequirements(OwnerComp, Context);
}

bool UBTTask_RunStateTree::CollectExternalData(const FStateTreeExecutionContext& Context, const UStateTree* StateTree, TArrayView<const FStateTreeExternalDataDesc> ExternalDataDescs, TArrayView<FStateTreeDataView> OutDataViews)
{
	return UStateTreeAIComponentSchema::CollectExternalData(Context, StateTree, ExternalDataDescs, OutDataViews);
}

TSubclassOf<UStateTreeSchema> UBTTask_RunStateTree::GetSchema() const
{
	return UStateTreeAIComponentSchema::StaticClass();
}

#if WITH_EDITOR
const UObject* UBTTask_RunStateTree::GetAssociatedAsset(TOptional<FBehaviorTreeNodeDebugContext> DebugContext) const
{
	return StateTreeRef.GetStateTree();
}
#endif // WITH_EDITOR
