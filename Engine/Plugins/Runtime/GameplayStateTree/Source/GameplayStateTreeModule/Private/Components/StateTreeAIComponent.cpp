// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/StateTreeAIComponent.h"

#include "Components/StateTreeAIComponentSchema.h"
#include "StateTreeExecutionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeAIComponent)

TSubclassOf<UStateTreeSchema> UStateTreeAIComponent::GetSchema() const
{
	return UStateTreeAIComponentSchema::StaticClass();
}
