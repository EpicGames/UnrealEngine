// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/StateTreeAIComponentSchema.h"

#include "AIController.h"
#include "BrainComponent.h"
#include "Conditions/StateTreeAIConditionBase.h"
#include "GameFramework/Pawn.h"
#include "StateTreeExecutionContext.h"
#include "Tasks/StateTreeAITask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeAIComponentSchema)

namespace UE::GameplayStateTree::Private
{
	static FLazyName Name_AIController = "AIController";
}

UStateTreeAIComponentSchema::UStateTreeAIComponentSchema(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
	: AIControllerClass(AAIController::StaticClass())
{
	check(ContextDataDescs.Num() == 1 && ContextDataDescs[0].Struct == AActor::StaticClass());
	// Make the Actor a pawn by default so it binds to the controlled pawn instead of the AIController.
	ContextActorClass = APawn::StaticClass();
	ContextDataDescs[0].Struct = ContextActorClass.Get();
	ContextDataDescs.Emplace(UE::GameplayStateTree::Private::Name_AIController, AIControllerClass.Get(), FGuid(0xEDB3CD97, 0x95F94E0A, 0xBD15207B, 0x98645CDC));
}

void UStateTreeAIComponentSchema::PostLoad()
{
	Super::PostLoad();
	ContextDataDescs[1].Struct = AIControllerClass.Get();
}

bool UStateTreeAIComponentSchema::IsStructAllowed(const UScriptStruct* InScriptStruct) const
{
	return Super::IsStructAllowed(InScriptStruct)
		|| InScriptStruct->IsChildOf(FStateTreeAITaskBase::StaticStruct())
		|| InScriptStruct->IsChildOf(FStateTreeAIConditionBase::StaticStruct());
}

void UStateTreeAIComponentSchema::SetContextData(FContextDataSetter& ContextDataSetter, bool bLogErrors) const
{
	const FName AIControllerName = UE::GameplayStateTree::Private::Name_AIController;
	ContextDataSetter.SetContextDataByName(AIControllerName, FStateTreeDataView(ContextDataSetter.GetComponent()->GetAIOwner()));

	Super::SetContextData(ContextDataSetter, bLogErrors);
}

#if WITH_EDITOR
void UStateTreeAIComponentSchema::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if (FProperty* Property = PropertyChangedEvent.Property)
	{
		if (Property->GetOwnerClass() == UStateTreeAIComponentSchema::StaticClass()
			&& Property->GetFName() == GET_MEMBER_NAME_CHECKED(UStateTreeAIComponentSchema, AIControllerClass))
		{
			ContextDataDescs[1].Struct = AIControllerClass.Get();
		}
	}
}
#endif // WITH_EDITOR
