// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/StateTreeComponent.h"
#include "GameplayTasksComponent.h"
#include "StateTreeExecutionContext.h"
#include "VisualLogger/VisualLogger.h"
#include "AIController.h"
#include "Components/StateTreeComponentSchema.h"
#include "Engine/World.h"
#include "Tasks/AITask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeComponent)

#define STATETREE_LOG(Verbosity, Format, ...) UE_VLOG_UELOG(GetOwner(), LogStateTree, Verbosity, Format, ##__VA_ARGS__)
#define STATETREE_CLOG(Condition, Verbosity, Format, ...) UE_CVLOG_UELOG((Condition), GetOwner(), LogStateTree, Verbosity, Format, ##__VA_ARGS__)

namespace UE::GameplayStateTree::Private
{
	bool bScheduledTickAllowed = true;
	static FAutoConsoleVariableRef CVarRuntimeValidationContext(
		TEXT("StateTree.Component.ScheduledTickEnabled"),
		bScheduledTickAllowed,
		TEXT("True if the scheduled tick feature is enabled for StateTreeComponent. A ScheduledTick StateTree can sleep or delayed for better performance.")
	);
}

//////////////////////////////////////////////////////////////////////////
// UStateTreeComponent

void FStateTreeComponentExecutionExtension::ScheduleNextTick(const FContextParameters& Context, const FNextTickArguments& Args)
{
	if (ensure(Component))
	{
		Component->ConditionalEnableTick();
	}
}

// //////////////////////////////////////////////////////////////////////////
// UStateTreeComponent

UStateTreeComponent::UStateTreeComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bWantsInitializeComponent = true;
	PrimaryComponentTick.bStartWithTickEnabled = false;
	bIsRunning = false;
	bIsPaused = false;
}

void UStateTreeComponent::InitializeComponent()
{
	// Skipping UBrainComponent
	UActorComponent::InitializeComponent();

	if (bStartLogicAutomatically)
	{
		ValidateStateTreeReference();
	}
}

#if WITH_EDITOR
void UStateTreeComponent::PostLoad()
{
	Super::PostLoad();
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (StateTree_DEPRECATED != nullptr)
	{
		StateTreeRef.SetStateTree(StateTree_DEPRECATED);
		StateTreeRef.SyncParameters();
		StateTree_DEPRECATED = nullptr;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif //WITH_EDITOR

void UStateTreeComponent::UninitializeComponent()
{
	// Skipping UBrainComponent
	UActorComponent::UninitializeComponent();
}

bool UStateTreeComponent::CollectExternalData(const FStateTreeExecutionContext& Context, const UStateTree* StateTree, TArrayView<const FStateTreeExternalDataDesc> ExternalDataDescs, TArrayView<FStateTreeDataView> OutDataViews) const
{
	return UStateTreeComponentSchema::CollectExternalData(Context, StateTree, ExternalDataDescs, OutDataViews);
}

bool UStateTreeComponent::SetContextRequirements(FStateTreeExecutionContext& Context, bool bLogErrors)
{
	Context.SetLinkedStateTreeOverrides(LinkedStateTreeOverrides);
	Context.SetCollectExternalDataCallback(FOnCollectStateTreeExternalData::CreateUObject(this, &UStateTreeComponent::CollectExternalData));
	return UStateTreeComponentSchema::SetContextRequirements(*this, Context, bLogErrors);
}

void UStateTreeComponent::BeginPlay()
{
	Super::BeginPlay();
	
	if (bStartLogicAutomatically)
	{
		StartLogic();
	}
}

void UStateTreeComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	StopLogic(UEnum::GetValueAsString(EndPlayReason));

	Super::EndPlay(EndPlayReason);
}

void UStateTreeComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (!bIsRunning || bIsPaused)
	{
		STATETREE_LOG(Warning, TEXT("%hs: Ticking a paused or a not running State Tree component."), __FUNCTION__);
		DisableTick();
		return;
	}

	if (!StateTreeRef.IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: Trying to tick State Tree component with invalid asset."), __FUNCTION__);
		bIsRunning = false;
		DisableTick();
		return;
	}

	if (CurrentlyRunningExecContext)
	{
		STATETREE_LOG(Error, TEXT("Reentrant call to %hs is not allowed."), __FUNCTION__);
		return;
	}

	FStateTreeExecutionContext Context(*GetOwner(), *StateTreeRef.GetStateTree(), InstanceData);
	TGuardValue<FStateTreeExecutionContext*> ReentrantExecutionContextGuard(CurrentlyRunningExecContext, &Context);

	const bool bValidContextRequirements = SetContextRequirements(Context);
	if (ensureMsgf(bValidContextRequirements, TEXT("The tree started with a valid context and it's now invalid.")))
	{
		const EStateTreeRunStatus PreviousRunStatus = Context.GetStateTreeRunStatus();
		const EStateTreeRunStatus CurrentRunStatus = Context.Tick(DeltaTime);

		ScheduleTickFrame(Context.GetNextScheduledTick());

		if (CurrentRunStatus != PreviousRunStatus)
		{
			OnStateTreeRunStatusChanged.Broadcast(CurrentRunStatus);
		}
	}
	else
	{
		STATETREE_LOG(Warning, TEXT("Context Requirements in %hs failed. Component tick is disabled."), __FUNCTION__);
		DisableTick();
	}
}

void UStateTreeComponent::StartLogic()
{
	STATETREE_LOG(Log, TEXT("%hs: Start Logic"), __FUNCTION__);
	StartTree();
}

void UStateTreeComponent::RestartLogic()
{
	STATETREE_LOG(Log, TEXT("%hs: Restart Logic"), __FUNCTION__);
	StartTree();
}

void UStateTreeComponent::StartTree()
{
	if (HasValidStateTreeReference().HasError())
	{
		bIsRunning = false;
		DisableTick();
		return;
	}

	if (CurrentlyRunningExecContext)
	{
		STATETREE_LOG(Error, TEXT("Reentrant call to %hs is not allowed."), __FUNCTION__);
		return;
	}

	FStateTreeExecutionContext Context(*GetOwner(), *StateTreeRef.GetStateTree(), InstanceData);
	TGuardValue<FStateTreeExecutionContext*> ReentrantExecutionContextGuard(CurrentlyRunningExecContext, &Context);
	if (SetContextRequirements(Context, /*bLogErrors*/true))
	{
		const EStateTreeRunStatus PreviousRunStatus = Context.GetStateTreeRunStatus();

		FStateTreeComponentExecutionExtension Extension;
		Extension.Component = this;
		const EStateTreeRunStatus CurrentRunStatus = Context.Start(FStateTreeExecutionContext::FStartParameters
			{
				.InitialGlobalParameters = StateTreeRef.GetGlobalParameters(),
				.ExecutionExtension = TInstancedStruct<FStateTreeComponentExecutionExtension>::Make(MoveTemp(Extension))
			});

		bIsRunning = CurrentRunStatus == EStateTreeRunStatus::Running;
		ScheduleTickFrame(Context.GetNextScheduledTick());
		if (CurrentRunStatus != PreviousRunStatus)
		{
			OnStateTreeRunStatusChanged.Broadcast(CurrentRunStatus);
		}
	}
	else
	{
		STATETREE_LOG(Warning, TEXT("Context Requirements in %hs failed. Component tick is disabled."), __FUNCTION__);
		DisableTick();
	}
}

void UStateTreeComponent::StopLogic(const FString& Reason)
{
	STATETREE_LOG(Log, TEXT("%hs: Stopping, reason: \'%s\'"), __FUNCTION__, *Reason);

	auto StopTree = [this](FStateTreeExecutionContext& Context)
	{
		bIsRunning = false;
		const EStateTreeRunStatus PreviousRunStatus = Context.GetStateTreeRunStatus();
		const EStateTreeRunStatus CurrentRunStatus = Context.Stop();

		// Note OnStateTreeRunStatusChanged can enable tick again.
		if (CurrentRunStatus != PreviousRunStatus)
		{
			OnStateTreeRunStatusChanged.Broadcast(CurrentRunStatus);
		}
	};

	if (!bIsRunning)
	{
		return;
	}

	DisableTick();

	if (HasValidStateTreeReference().HasError())
	{
		bIsRunning = false;
		STATETREE_LOG(Warning, TEXT("%hs: Trying to stop State Tree component with invalid asset."), __FUNCTION__);
		return;
	}

	if (CurrentlyRunningExecContext)
	{
		// Tree Stop will be delayed to the end of the frame in a reentrant call, but we need to use the existing execution context
		StopTree(*CurrentlyRunningExecContext);
	}
	else
	{
		FStateTreeExecutionContext Context(*GetOwner(), *StateTreeRef.GetStateTree(), InstanceData);
		if (SetContextRequirements(Context))
		{
			StopTree(Context);
		}
		else
		{
			STATETREE_LOG(Warning, TEXT("Context Requirements in %hs failed. Component tick is disabled."), __FUNCTION__);
		}
	}
}

void UStateTreeComponent::Cleanup()
{
	StopLogic(TEXT("Cleanup"));
}

void UStateTreeComponent::PauseLogic(const FString& Reason)
{
	STATETREE_LOG(Log, TEXT("%hs: Execution updates: PAUSED (%s)"), __FUNCTION__, *Reason);
	bIsPaused = true;

	DisableTick();
}

EAILogicResuming::Type UStateTreeComponent::ResumeLogic(const FString& Reason)
{
	STATETREE_LOG(Log, TEXT("%hs: Execution updates: RESUMED (%s)"), __FUNCTION__, *Reason);

	const EAILogicResuming::Type SuperResumeResult = Super::ResumeLogic(Reason);

	bIsPaused = false;
	if (bIsRunning)
	{
		FStateTreeMinimalExecutionContext Context(GetOwner(), StateTreeRef.GetStateTree(), InstanceData);
		ScheduleTickFrame(Context.GetNextScheduledTick());
	}
	else
	{
		DisableTick();
	}

	return SuperResumeResult;
}

void UStateTreeComponent::ScheduleTickFrame(const FStateTreeScheduledTick& NextTick)
{
	if (bIsRunning && !bIsPaused)
	{
		if (!UE::GameplayStateTree::Private::bScheduledTickAllowed)
		{
			// Make sure the component tick is enabled. It ticks every frame.
			if (!IsComponentTickEnabled())
			{
				SetComponentTickEnabled(true);
			}
			return;
		}

		if (NextTick.ShouldSleep())
		{
			if (IsComponentTickEnabled())
			{
				SetComponentTickEnabled(false);
			}
		}
		else
		{
			if (!IsComponentTickEnabled())
			{
				SetComponentTickEnabled(true);
			}

			if (NextTick.ShouldTickEveryFrames())
			{
				SetComponentTickIntervalAndCooldown(0.0f);
			}
			else
			{
				// We need to force a small dt to tell the TickTaskManager we might not want to be tick every frame.
				constexpr float FORCE_TICK_INTERVAL_DT = UE_KINDA_SMALL_NUMBER;
				const float NextTickDeltaTime = !NextTick.ShouldTickOnceNextFrame() ? NextTick.GetTickRate() : FORCE_TICK_INTERVAL_DT;
				if (!FMath::IsNearlyEqual(GetComponentTickInterval(), NextTickDeltaTime))
				{
					SetComponentTickIntervalAndCooldown(NextTickDeltaTime);
				}
			}
		}
	}
	else
	{
		DisableTick();
	}
}

void UStateTreeComponent::ConditionalEnableTick()
{
	STATETREE_LOG(Log, TEXT("%hs: EnabledTick manually."), __FUNCTION__);
	ScheduleTickFrame(FStateTreeScheduledTick::MakeNextFrame());
}

void UStateTreeComponent::DisableTick()
{
	if (IsComponentTickEnabled())
	{
		SetComponentTickEnabled(false);
	}
}

bool UStateTreeComponent::IsRunning() const
{
	return bIsRunning;
}

bool UStateTreeComponent::IsPaused() const
{
	return bIsPaused;
}

UGameplayTasksComponent* UStateTreeComponent::GetGameplayTasksComponent(const UGameplayTask& Task) const
{
	const UAITask* AITask = Cast<const UAITask>(&Task);
	return (AITask && AITask->GetAIController()) ? AITask->GetAIController()->GetGameplayTasksComponent(Task) : Task.GetGameplayTasksComponent();
}

AActor* UStateTreeComponent::GetGameplayTaskOwner(const UGameplayTask* Task) const
{
	if (Task == nullptr)
	{
		return GetAIOwner();
	}

	if (const UAITask* AITask = Cast<const UAITask>(Task))
	{
		return AITask->GetAIController();
	}

	const UGameplayTasksComponent* TasksComponent = Task->GetGameplayTasksComponent();
	return TasksComponent ? TasksComponent->GetGameplayTaskOwner(Task) : nullptr;
}

AActor* UStateTreeComponent::GetGameplayTaskAvatar(const UGameplayTask* Task) const
{
	if (Task == nullptr)
	{
		return GetAIOwner() ? GetAIOwner()->GetPawn() : nullptr;
	}

	if (const UAITask* AITask = Cast<const UAITask>(Task))
	{
		return AITask->GetAIController() ? AITask->GetAIController()->GetPawn() : nullptr;
	}

	const UGameplayTasksComponent* TasksComponent = Task->GetGameplayTasksComponent();
	return TasksComponent ? TasksComponent->GetGameplayTaskAvatar(Task) : nullptr;
}

uint8 UStateTreeComponent::GetGameplayTaskDefaultPriority() const
{
	return static_cast<uint8>(EAITaskPriority::AutonomousAI);
}

void UStateTreeComponent::OnGameplayTaskInitialized(UGameplayTask& Task)
{
	const UAITask* AITask = Cast<const UAITask>(&Task);
	if (AITask && (AITask->GetAIController() == nullptr))
	{
		// this means that the task has either been created without specifying 
		// UAITask::OwnerController's value (like via BP's Construct Object node)
		// or it has been created in C++ with inappropriate function
		UE_LOGF(LogStateTree, Error, "Missing AIController in AITask %ls", *AITask->GetName());
	}
}

TSubclassOf<UStateTreeSchema> UStateTreeComponent::GetSchema() const
{
	return UStateTreeComponentSchema::StaticClass();
}

void UStateTreeComponent::ValidateStateTreeReference()
{
 	TValueOrError<void, FString> ValidResult = HasValidStateTreeReference();
 	if (ValidResult.HasError())
 	{
 	 	STATETREE_LOG(Error, TEXT("%hs: %s. Cannot initialize."), __FUNCTION__, * ValidResult.GetError());
 	 	return;
 	}

	FStateTreeReadOnlyExecutionContext Context(GetOwner(), StateTreeRef.GetStateTree(), InstanceData);
	if (!Context.IsValid())
	{
		STATETREE_LOG(Error, TEXT("%hs: Failed to init StateTreeContext."), __FUNCTION__);
		return;
	}
}

TValueOrError<void, FString> UStateTreeComponent::HasValidStateTreeReference() const
{
	if (!StateTreeRef.IsValid())
	{
		return MakeError(TEXT("The State Tree asset is not set."));
	}

	if (!StateTreeRef.GetStateTree()->IsReadyToRun())
	{
		return MakeError(TEXT("The State Tree schema is not ready to run."));
	}

	if (StateTreeRef.GetStateTree()->GetSchema() == nullptr
		|| !StateTreeRef.GetStateTree()->GetSchema()->GetClass()->IsChildOf(UStateTreeComponentSchema::StaticClass()))
	{
		return MakeError(TEXT("The State Tree schema is not compatible."));
	}

	return MakeValue();
}

void UStateTreeComponent::SetStateTree(UStateTree* InStateTree)
{
	if (const UStateTree* ActiveStateTree = StateTreeRef.GetStateTree())
	{
		// Can't change the StateTreeRef on a running tree. It might change the instance data while running tasks.
		FStateTreeReadOnlyExecutionContext Context(GetOwner(), ActiveStateTree, InstanceData);
		if (Context.GetStateTreeRunStatus() == EStateTreeRunStatus::Running)
		{
			STATETREE_LOG(Warning, TEXT("%hs : Trying to change the state tree on a running instance."), __FUNCTION__);
			return;
		}
	}

	StateTreeRef.SetStateTree(InStateTree);

	if (StateTreeRef.GetStateTree() != nullptr)
	{
		ValidateStateTreeReference();
	}
}

void UStateTreeComponent::SetStateTreeReference(FStateTreeReference InStateTreeReference)
{
	if (const UStateTree* const ActiveStateTree = StateTreeRef.GetStateTree())
	{
		// Can't change the StateTreeRef on a running tree. It might change the instance data while running tasks.
		FStateTreeReadOnlyExecutionContext Context(GetOwner(), StateTreeRef.GetStateTree(), InstanceData);
		if (Context.GetStateTreeRunStatus() == EStateTreeRunStatus::Running)
		{
			STATETREE_LOG(Warning, TEXT("%hs : Trying to change the state tree on a running instance."), __FUNCTION__);
			return;
		}
	}

	StateTreeRef = MoveTemp(InStateTreeReference);

	if (StateTreeRef.GetStateTree() != nullptr)
	{
		ValidateStateTreeReference();
	}
}

void UStateTreeComponent::SetLinkedStateTreeOverrides(FStateTreeReferenceOverrides Overrides)
{
	// Validate the schema
	TSubclassOf<UStateTreeComponentSchema> BaseSchema = UStateTreeComponentSchema::StaticClass();
	if (StateTreeRef.GetStateTree() && StateTreeRef.GetStateTree()->GetSchema())
	{
		BaseSchema = StateTreeRef.GetStateTree()->GetSchema()->GetClass();
	}

	for (const FStateTreeReferenceOverrideItem& Item : Overrides.GetOverrideItems())
	{
		if (const UStateTree* ItemStateTree = Item.GetStateTreeReference().GetStateTree())
		{
			if (ItemStateTree->GetSchema() == nullptr
				|| !ItemStateTree->GetSchema()->GetClass()->IsChildOf(BaseSchema.Get()))
			{
				STATETREE_LOG(Warning, TEXT("%hs: Trying to set the linked overrides '%s' with a wrong schema. %s."),
					__FUNCTION__,
					*Item.GetStateTag().ToString(),
					*ItemStateTree->GetFullName()
					);
				return;
			}
		}
	}

	LinkedStateTreeOverrides = MoveTemp(Overrides);
}

void UStateTreeComponent::AddLinkedStateTreeOverrides(const FGameplayTag StateTag, FStateTreeReference StateTreeReference)
{
	// Validate the schema
	if (const UStateTree* ItemStateTree = StateTreeReference.GetStateTree())
	{
		TSubclassOf<UStateTreeComponentSchema> BaseSchema = UStateTreeComponentSchema::StaticClass();
		if (StateTreeRef.GetStateTree() && StateTreeRef.GetStateTree()->GetSchema())
		{
			BaseSchema = StateTreeRef.GetStateTree()->GetSchema()->GetClass();
		}

		if (ItemStateTree->GetSchema() == nullptr
			|| !ItemStateTree->GetSchema()->GetClass()->IsChildOf(BaseSchema.Get()))
		{
			STATETREE_LOG(Warning, TEXT("%hs: Trying to set the linked overrides with the wrong schema. %s."), __FUNCTION__, *ItemStateTree->GetFullName());
			return;
		}
	}
	LinkedStateTreeOverrides.AddOverride(FStateTreeReferenceOverrideItem(StateTag, MoveTemp(StateTreeReference)));
}

void UStateTreeComponent::RemoveLinkedStateTreeOverrides(const FGameplayTag StateTag)
{
	LinkedStateTreeOverrides.RemoveOverride(StateTag);
}

void UStateTreeComponent::SetStartLogicAutomatically(const bool bInStartLogicAutomatically)
{
	bStartLogicAutomatically = bInStartLogicAutomatically;
}

void UStateTreeComponent::SendStateTreeEvent(const FStateTreeEvent& Event)
{
	SendStateTreeEvent(Event.Tag, Event.Payload, Event.Origin);
}

void UStateTreeComponent::SendStateTreeEvent(const FGameplayTag Tag, const FConstStructView Payload, const FName Origin)
{
	if (!bIsRunning)
	{
		STATETREE_LOG(Warning, TEXT("%hs: Trying to send event to a State Tree that is not started yet."), __FUNCTION__);
		return;
	}

	if (!StateTreeRef.IsValid())
	{
		STATETREE_LOG(Warning, TEXT("%hs: Trying to send event to State Tree component with invalid asset."), __FUNCTION__);
		return;
	}

	FStateTreeMinimalExecutionContext Context(GetOwner(), StateTreeRef.GetStateTree(), InstanceData);
	Context.SendEvent(Tag, Payload, Origin);
}

EStateTreeRunStatus UStateTreeComponent::GetStateTreeRunStatus() const
{
	if (const FStateTreeExecutionState* Exec = InstanceData.GetExecutionState())
	{
		return Exec->TreeRunStatus;
	}

	return EStateTreeRunStatus::Failed;
}

#if WITH_GAMEPLAY_DEBUGGER
FString UStateTreeComponent::GetDebugInfoString() const
{
	if (!StateTreeRef.IsValid())
	{
		return FString("No StateTree to run.");
	}

	return FConstStateTreeExecutionContextView(*GetOwner(), *StateTreeRef.GetStateTree(), InstanceData).Get().GetDebugInfoString();
}

TArray<FName> UStateTreeComponent::GetActiveStateNames() const
{
	if (!StateTreeRef.IsValid())
	{
		return TArray<FName>();
	}

	return FConstStateTreeExecutionContextView(*GetOwner(), *StateTreeRef.GetStateTree(), InstanceData).Get().GetActiveStateNames();
}
#endif // WITH_GAMEPLAY_DEBUGGER

#undef STATETREE_LOG
#undef STATETREE_CLOG
