// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/StateTreeComponentSchema.h"
#include "AIController.h"
#include "BrainComponent.h"
#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "StateTreeConditionBase.h"
#include "StateTreeConsiderationBase.h"
#include "StateTreeEvaluatorBase.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeTaskBase.h"
#include "Subsystems/WorldSubsystem.h"
#include "Tasks/StateTreeAITask.h"
#include "VisualLogger/VisualLogger.h"
#include "StateTreePropertyFunctionBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeComponentSchema)

namespace UE::GameplayStateTree::Private
{
	static FLazyName Name_Actor = "Actor";

	FAutoConsoleVariable CVarDefaultScheduledTickAllowed(
		TEXT("StateTree.Component.DefaultScheduledTickAllowed"),
		true,
		TEXT("By default, allow execution context to sleep and the tick delayed.")
	);
}

UStateTreeComponentSchema::UStateTreeComponentSchema()
	: ContextActorClass(AActor::StaticClass())
	, ContextDataDescs({{ UE::GameplayStateTree::Private::Name_Actor, AActor::StaticClass(), FGuid(0x1D971B00, 0x28884FDE, 0xB5436802, 0x36984FD5) }})
{
}

bool UStateTreeComponentSchema::IsStructAllowed(const UScriptStruct* InScriptStruct) const
{
	return InScriptStruct->IsChildOf(FStateTreeConditionCommonBase::StaticStruct())
	|| InScriptStruct->IsChildOf(FStateTreeEvaluatorCommonBase::StaticStruct())
	|| InScriptStruct->IsChildOf(FStateTreeTaskCommonBase::StaticStruct())
	|| InScriptStruct->IsChildOf(FStateTreeConsiderationCommonBase::StaticStruct())
	|| InScriptStruct->IsChildOf(FStateTreePropertyFunctionCommonBase::StaticStruct());
}

bool UStateTreeComponentSchema::IsClassAllowed(const UClass* InClass) const
{
	return IsChildOfBlueprintBase(InClass);
}

bool UStateTreeComponentSchema::IsExternalItemAllowed(const UStruct& InStruct) const
{
	return InStruct.IsChildOf(AActor::StaticClass())
			|| InStruct.IsChildOf(UActorComponent::StaticClass())
			|| InStruct.IsChildOf(UWorldSubsystem::StaticClass());
}

bool UStateTreeComponentSchema::IsScheduledTickAllowed() const
{
	switch (ScheduledTickPolicy)
	{
	case EStateTreeComponentSchemaScheduledTickPolicy::Default:
	default:
		return UE::GameplayStateTree::Private::CVarDefaultScheduledTickAllowed->GetBool();
	case EStateTreeComponentSchemaScheduledTickPolicy::Denied:
		return false;
	case EStateTreeComponentSchemaScheduledTickPolicy::Allowed:
		return true;
	}
}

TConstArrayView<FStateTreeExternalDataDesc> UStateTreeComponentSchema::GetContextDataDescs() const
{
	return ContextDataDescs;
}

void UStateTreeComponentSchema::PostLoad()
{
	Super::PostLoad();
	GetContextActorDataDesc().Struct = ContextActorClass.Get() ? ContextActorClass.Get() : AActor::StaticClass();
}

#if WITH_EDITOR
void UStateTreeComponentSchema::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	FProperty* Property = PropertyChangedEvent.Property;

	if (Property)
	{
		if (Property->GetOwnerClass() == UStateTreeComponentSchema::StaticClass()
			&& Property->GetFName() == GET_MEMBER_NAME_CHECKED(UStateTreeComponentSchema, ContextActorClass))
		{
			GetContextActorDataDesc().Struct = ContextActorClass.Get() ? ContextActorClass.Get() : AActor::StaticClass();
		}
	}
}
#endif

UStateTreeComponentSchema::FContextDataSetter::FContextDataSetter(TNotNull<const UBrainComponent*> InBrainComponent, FStateTreeExecutionContext& InContext)
	: BrainComponent(InBrainComponent)
	, ExecutionContext(InContext)
{}

bool UStateTreeComponentSchema::FContextDataSetter::SetContextDataByName(FName Name, FStateTreeDataView DataView)
{
	return ExecutionContext.IsValid() ? ExecutionContext.SetContextDataByName(Name, DataView) : false;
}

TNotNull<const UStateTree*> UStateTreeComponentSchema::FContextDataSetter::GetStateTree() const
{
	return ExecutionContext.GetStateTree();
}

TNotNull<const UStateTreeComponentSchema*> UStateTreeComponentSchema::FContextDataSetter::GetSchema() const
{
	return Cast<UStateTreeComponentSchema>(ExecutionContext.GetStateTree()->GetSchema());
}

void UStateTreeComponentSchema::SetContextData(FContextDataSetter& ContextDataSetter, bool bLogErrors) const
{
	// Make sure the actor matches one required.
	AActor* ContextActor = nullptr;
	const UStateTreeComponentSchema* Schema = ContextDataSetter.GetSchema();
	if (Schema)
	{
		AAIController* AIOwner = ContextDataSetter.GetComponent()->GetAIOwner();
		if (AAIController* OwnerController = (AIOwner != nullptr) ? AIOwner : Cast<AAIController>(ContextDataSetter.GetComponent()->GetOwner()))
		{
			if (OwnerController && OwnerController->IsA(Schema->GetContextActorClass()))
			{
				ContextActor = OwnerController;
			}
		}
		if (ContextActor == nullptr)
		{
			if (AActor* OwnerActor = (AIOwner != nullptr) ? AIOwner->GetPawn().Get() : ContextDataSetter.GetComponent()->GetOwner())
			{
				if (OwnerActor && OwnerActor->IsA(Schema->GetContextActorClass()))
				{
					ContextActor = OwnerActor;
				}
			}
		}
		if (ContextActor == nullptr && bLogErrors)
		{
			UE_VLOG_UELOG(ContextDataSetter.GetComponent()->GetOwner(), LogStateTree, Error, TEXT("%hs: Could not find context actor of type %s. StateTree will not update."), __FUNCTION__, *GetNameSafe(Schema->GetContextActorClass()));
		}
	}
	else if (bLogErrors)
	{
		UE_VLOG_UELOG(ContextDataSetter.GetComponent()->GetOwner(), LogStateTree, Error, TEXT("%hs: Expected StateTree asset to contain StateTreeComponentSchema. StateTree will not update."), __FUNCTION__);
	}

	const FName ActorName = UE::GameplayStateTree::Private::Name_Actor;
	ContextDataSetter.SetContextDataByName(ActorName, FStateTreeDataView(ContextActor));
}

bool UStateTreeComponentSchema::SetContextRequirements(UBrainComponent& BrainComponent, FStateTreeExecutionContext& Context, bool bLogErrors /*= false*/)
{
	if (!Context.IsValid())
	{
		return false;
	}

	FContextDataSetter ContextDataSetter = FContextDataSetter(&BrainComponent, Context);
	ContextDataSetter.GetSchema()->SetContextData(ContextDataSetter, bLogErrors);

	bool bResult = Context.AreContextDataViewsValid();
	if (!bResult && bLogErrors)
	{
		UE_VLOG_UELOG(BrainComponent.GetOwner(), LogStateTree, Error, TEXT("%hs: Missing external data requirements. StateTree will not update."), __FUNCTION__);
	}

	return bResult;
}

bool UStateTreeComponentSchema::CollectExternalData(const FStateTreeExecutionContext& Context, const UStateTree* StateTree, TArrayView<const FStateTreeExternalDataDesc> ExternalDataDescs, TArrayView<FStateTreeDataView> OutDataViews)
{
	checkf(ExternalDataDescs.Num() == OutDataViews.Num(), TEXT("The execution context failed to fill OutDataViews with empty values."));

	const UWorld* World = Context.GetWorld();
	if (World == nullptr)
	{
		UE_VLOG_UELOG(Context.GetOwner(), LogStateTree, Error, TEXT("%hs: External data can't find the world ('%s' using StateTree '%s')."),
			__FUNCTION__, *GetNameSafe(Context.GetOwner()), *GetFullNameSafe(Context.GetStateTree()));
		return false;
	}

	AActor* Owner = Cast<AActor>(Context.GetOwner());
	if (!Owner)
	{
		UE_VLOG_UELOG(Context.GetOwner(), LogStateTree, Error, TEXT("%hs: The external data owner is invalid ('%s' using StateTree '%s')."),
			__FUNCTION__, *GetNameSafe(Context.GetOwner()), *GetFullNameSafe(Context.GetStateTree()));
		return false;
	}

	int32 IssuesFoundCounter = 0;
	AAIController* AIOwner = Cast<AAIController>(Owner);
	for (int32 Index = 0; Index < ExternalDataDescs.Num(); Index++)
	{
		const FStateTreeExternalDataDesc& ItemDesc = ExternalDataDescs[Index];
		if (ItemDesc.Struct != nullptr)
		{
			if (ItemDesc.Struct->IsChildOf(UWorldSubsystem::StaticClass()))
			{
				UWorldSubsystem* Subsystem = World->GetSubsystemBase(Cast<UClass>(const_cast<UStruct*>(ItemDesc.Struct.Get())));
				OutDataViews[Index] = FStateTreeDataView(Subsystem);
				UE_CVLOG(Subsystem == nullptr, Context.GetOwner(), LogStateTree, Error, TEXT("StateTree %s: Could not find required subsystem %s"), *GetNameSafe(Context.GetStateTree()), *GetNameSafe(ItemDesc.Struct));
				IssuesFoundCounter += Subsystem != nullptr ? 0 : 1;
			}
			else if (ItemDesc.Struct->IsChildOf(UActorComponent::StaticClass()))
			{
				UActorComponent* Component = Owner->FindComponentByClass(Cast<UClass>(const_cast<UStruct*>(ItemDesc.Struct.Get())));
				OutDataViews[Index] = FStateTreeDataView(Component);
				UE_CVLOG(Component == nullptr, Context.GetOwner(), LogStateTree, Error, TEXT("StateTree %s: Could not find required component %s"), *GetNameSafe(Context.GetStateTree()), *GetNameSafe(ItemDesc.Struct));
				IssuesFoundCounter += Component != nullptr ? 0 : 1;
			}
			else if (ItemDesc.Struct->IsChildOf(APawn::StaticClass()))
			{
				APawn* OwnerPawn = (AIOwner != nullptr) ? AIOwner->GetPawn().Get() : Cast<APawn>(Owner);
				UE_CVLOG(OwnerPawn == nullptr, Context.GetOwner(), LogStateTree, Error, TEXT("StateTree %s: Could not find required pawn %s"), *GetNameSafe(Context.GetStateTree()), *GetNameSafe(ItemDesc.Struct));
				OutDataViews[Index] = FStateTreeDataView(OwnerPawn);
				IssuesFoundCounter += OwnerPawn != nullptr ? 0 : 1;
			}
			else if (ItemDesc.Struct->IsChildOf(AAIController::StaticClass()))
			{
				AAIController* OwnerController = AIOwner;
				UE_CVLOG(OwnerController == nullptr, Context.GetOwner(), LogStateTree, Error, TEXT("StateTree %s: Could not find required controller %s"), *GetNameSafe(Context.GetStateTree()), *GetNameSafe(ItemDesc.Struct));
				OutDataViews[Index] = FStateTreeDataView(OwnerController);
				IssuesFoundCounter += OwnerController != nullptr ? 0 : 1;
			}
			else if (ItemDesc.Struct->IsChildOf(AActor::StaticClass()))
			{
				AActor* OwnerActor = (AIOwner != nullptr) ? AIOwner->GetPawn().Get() : Owner;
				UE_CVLOG(OwnerActor == nullptr, Context.GetOwner(), LogStateTree, Error, TEXT("StateTree %s: Could not find required actor %s"), *GetNameSafe(Context.GetStateTree()), *GetNameSafe(ItemDesc.Struct));
				OutDataViews[Index] = FStateTreeDataView(OwnerActor);
				IssuesFoundCounter += OwnerActor != nullptr ? 0 : 1;
			}
		}
	}

	return IssuesFoundCounter == 0;
}
