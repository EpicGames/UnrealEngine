// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayStateTreeBlueprintFunctionLibrary.h"
#include "AIController.h"
#include "Blueprint/AIBlueprintHelperLibrary.h"
#include "Components/StateTreeComponent.h"
#include "StateTree.h"
#include "StateTreeTypes.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayStateTreeBlueprintFunctionLibrary)

bool UGameplayStateTreeBlueprintFunctionLibrary::RunStateTree(AActor* Actor, UStateTree* StateTreeAsset)
{
	if (Actor == nullptr)
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Unable to run an instance of statetree '%s': null actor")
			, *GetNameSafe(StateTreeAsset))
			, ELogVerbosity::Warning);
		return false;
	}

	if (StateTreeAsset == nullptr)
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Unable to run an instance of statetree on actor '%s': null state tree asset")
			, *Actor->GetName())
			, ELogVerbosity::Warning);
		return false;
	}

	AAIController* AIController = Cast<AAIController>(Actor);
	UStateTreeComponent* StateTreeComponent = nullptr;
	if (AIController != nullptr)
	{
		StateTreeComponent = Cast<UStateTreeComponent>(AIController->GetBrainComponent());
	}
	else
	{
		StateTreeComponent = Actor->GetComponentByClass<UStateTreeComponent>();
	}

	if (StateTreeComponent == nullptr)
	{
		StateTreeComponent = NewObject<UStateTreeComponent>(Actor);

		// Make sure to not attempt to start the logic until the component is set up
		StateTreeComponent->SetStartLogicAutomatically(false);
		StateTreeComponent->RegisterComponent();
		REDIRECT_OBJECT_TO_VLOG(StateTreeComponent, Actor);
	}

	// Make sure BrainComponent points at the newly created StateTree component
	if (AIController != nullptr)
	{
		AIController->BrainComponent = StateTreeComponent;
	}

	check(StateTreeComponent);
	if (StateTreeComponent->IsRunning())
	{
		StateTreeComponent->StopLogic(TEXT("Starting logic with new asset"));
	}

	StateTreeComponent->SetStateTree(StateTreeAsset);
	StateTreeComponent->SetStartLogicAutomatically(true);

	// If the component was already simulating we start the logic, otherwise we set it to auto-start on BeginPlay
	if (StateTreeComponent->HasBegunPlay())
	{
		StateTreeComponent->StartLogic();
	}

	return true;
}