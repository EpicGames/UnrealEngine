// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayStateTreeModule.h"
#include "Modules/ModuleManager.h"


//----------------------------------------------------------------------//
// IGameplayStateTreeModule
//----------------------------------------------------------------------//
IGameplayStateTreeModule& IGameplayStateTreeModule::Get()
{
	return FModuleManager::LoadModuleChecked<IGameplayStateTreeModule>("GameplayStateTreeModule");
}

bool IGameplayStateTreeModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded("GameplayStateTreeModule");
}


//----------------------------------------------------------------------//
// FGameplayStateTreeModule
//----------------------------------------------------------------------//
class FGameplayStateTreeModule : public IGameplayStateTreeModule
{
};

IMPLEMENT_MODULE(FGameplayStateTreeModule, GameplayStateTreeModule)
