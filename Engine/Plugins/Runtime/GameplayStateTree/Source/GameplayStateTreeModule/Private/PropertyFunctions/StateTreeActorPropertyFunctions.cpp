// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyFunctions/StateTreeActorPropertyFunctions.h"

#include "GameFramework/Actor.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeNodeDescriptionHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeActorPropertyFunctions)

#define LOCTEXT_NAMESPACE "StateTree"

void FStateTreeGetActorLocationPropertyFunction::Execute(FStateTreeExecutionContext& Context) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	InstanceData.Output = InstanceData.Input ? InstanceData.Input->GetActorLocation() : FVector::ZeroVector;
}

#if WITH_EDITOR
FText FStateTreeGetActorLocationPropertyFunction::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	return UE::StateTree::DescHelpers::GetDescriptionForSingleParameterFunc<FInstanceDataType>(LOCTEXT("StateTreeActorLocation", "GetActorLocation"), ID, InstanceDataView, BindingLookup, Formatting);
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
