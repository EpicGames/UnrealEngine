// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreePropertyFunctionBase.h"

#include "StateTreeActorPropertyFunctions.generated.h"

USTRUCT()
struct FStateTreeGetActorLocationPropertyFunctionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Parameter)
	TObjectPtr<AActor> Input = nullptr;

	UPROPERTY(EditAnywhere, Category = Output)
	FVector Output = FVector::ZeroVector;
};

USTRUCT(DisplayName = "Get Actor Location")
struct FStateTreeGetActorLocationPropertyFunction : public FStateTreePropertyFunctionCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeGetActorLocationPropertyFunctionInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	virtual void Execute(FStateTreeExecutionContext& Context) const override;

#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const;
#endif
};