// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/StateTreeMoveToTask.h"

#include "AIController.h"
#include "Navigation/PathFollowingComponent.h"
#include "StateTreeAsyncExecutionContext.h"
#include "StateTreeExecutionContext.h"
#include "Tasks/AITask_MoveTo.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeMoveToTask)

#define LOCTEXT_NAMESPACE "GameplayStateTree"

FStateTreeMoveToTask::FStateTreeMoveToTask()
{
	bShouldCallTick = false;
	bShouldCopyBoundPropertiesOnTick = false;
	bShouldCopyBoundPropertiesOnExitState = false;
}

bool FStateTreeMoveToTask::Link(FStateTreeLinker& Linker)
{
	bShouldCallTick = bSavedShouldCallTick;
	bShouldCopyBoundPropertiesOnTick = bSavedShouldCallTick;

	return Super::Link(Linker);
}

EStateTreeRunStatus FStateTreeMoveToTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	if (!InstanceData.AIController)
	{
		UE_VLOG(Context.GetOwner(), LogStateTree, Error, TEXT("FStateTreeMoveToTask failed since AIController is missing."));
		return EStateTreeRunStatus::Failed;
	}

	InstanceData.TaskOwner = TScriptInterface<IGameplayTaskOwnerInterface>(InstanceData.AIController->FindComponentByInterface(UGameplayTaskOwnerInterface::StaticClass()));
	if (!InstanceData.TaskOwner)
	{
		InstanceData.TaskOwner = InstanceData.AIController;
	}

	return PerformMoveTask(Context, *InstanceData.AIController);
}

EStateTreeRunStatus FStateTreeMoveToTask::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	if (InstanceData.MoveToTask)
	{
		if (InstanceData.bTrackMovingGoal && !InstanceData.TargetActor)
		{
			const FVector CurrentDestination = InstanceData.MoveToTask->GetMoveRequestRef().GetDestination();
			if (FVector::DistSquared(CurrentDestination, InstanceData.Destination) > (InstanceData.DestinationMoveTolerance * InstanceData.DestinationMoveTolerance))
			{
				UE_VLOG(Context.GetOwner(), LogStateTree, Log, TEXT("FStateTreeMoveToTask destination has moved enough. Restarting task."));
				return PerformMoveTask(Context, *InstanceData.AIController);
			}
		}

		return EStateTreeRunStatus::Running;
	}

	return EStateTreeRunStatus::Failed;
}

void FStateTreeMoveToTask::ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	if (InstanceData.MoveToTask && InstanceData.MoveToTask->GetState() != EGameplayTaskState::Finished)
	{
		UE_VLOG(Context.GetOwner(), LogStateTree, Log, TEXT("FStateTreeMoveToTask aborting move to because state finished."));
		InstanceData.MoveToTask->ExternalCancel();
	}

	// todo: remove this once we fixed the instance data retention issue for re-entering state
	InstanceData.MoveToTask = nullptr;
}

UAITask_MoveTo* FStateTreeMoveToTask::PrepareMoveToTask(FStateTreeExecutionContext& Context, AAIController& Controller, UAITask_MoveTo* ExistingTask, FAIMoveRequest& MoveRequest) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	UAITask_MoveTo* MoveTask = ExistingTask ? ExistingTask : UAITask::NewAITask<UAITask_MoveTo>(Controller, *InstanceData.TaskOwner, EAITaskPriority::High);
	if (MoveTask)
	{
		MoveTask->SetUp(&Controller, MoveRequest);
	}

	return MoveTask;
}

EStateTreeRunStatus FStateTreeMoveToTask::PerformMoveTask(FStateTreeExecutionContext& Context, AAIController& Controller) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	FAIMoveRequest MoveReq;
	MoveReq.SetNavigationFilter(InstanceData.FilterClass ? InstanceData.FilterClass : Controller.GetDefaultNavigationFilterClass())
		.SetAllowPartialPath(InstanceData.bAllowPartialPath)
		.SetAcceptanceRadius(InstanceData.AcceptableRadius)
		.SetCanStrafe(InstanceData.bAllowStrafe)
		.SetReachTestIncludesAgentRadius(InstanceData.bReachTestIncludesAgentRadius)
		.SetReachTestIncludesGoalRadius(InstanceData.bReachTestIncludesGoalRadius)
		.SetRequireNavigableEndLocation(InstanceData.bRequireNavigableEndLocation)
		.SetProjectGoalLocation(InstanceData.bProjectGoalLocation)
		.SetUsePathfinding(true);

	if (InstanceData.TargetActor)
	{
		if (InstanceData.bTrackMovingGoal)
		{
			MoveReq.SetGoalActor(InstanceData.TargetActor);
		}
		else
		{
			MoveReq.SetGoalLocation(InstanceData.TargetActor->GetActorLocation());
		}
	}
	else
	{
		MoveReq.SetGoalLocation(InstanceData.Destination);
	}

	if (MoveReq.IsValid())
	{	
		InstanceData.MoveToTask = PrepareMoveToTask(Context, Controller, InstanceData.MoveToTask, MoveReq);
		if (InstanceData.MoveToTask)
		{
			const bool bIsGameplayTaskAlreadyActive = InstanceData.MoveToTask->IsActive();
			if (bIsGameplayTaskAlreadyActive)
			{
				InstanceData.MoveToTask->ConditionalPerformMove();
			}
			else
			{
				InstanceData.MoveToTask->ReadyForActivation();
			}

			// if it is already active, don't re-register the callback and wait for the callback to finish task
			if (!bIsGameplayTaskAlreadyActive)
			{
				// @todo: we want to check the state first time in case the task is a temporary task and the gameplay task is finished instantly, that WeakContext
				// won't be able to find the active frame/state. Remove this once we support that.
				if (InstanceData.MoveToTask->GetState() == EGameplayTaskState::Finished)
				{
					return InstanceData.MoveToTask->WasMoveSuccessful() ? EStateTreeRunStatus::Succeeded : EStateTreeRunStatus::Failed;
				}

				InstanceData.MoveToTask->OnMoveTaskFinished.AddLambda(
					[WeakContext = Context.MakeWeakExecutionContext()]
					(TEnumAsByte<EPathFollowingResult::Type> InResult, AAIController* InController)
					{
						WeakContext.FinishTask(InResult == EPathFollowingResult::Success ? EStateTreeFinishTaskType::Succeeded : EStateTreeFinishTaskType::Failed);
					});
			}

			return EStateTreeRunStatus::Running;
		}
	}

	UE_VLOG(Context.GetOwner(), LogStateTree, Error, TEXT("FStateTreeMoveToTask failed because it doesn't have a destination."));
	return EStateTreeRunStatus::Failed;
}

#if WITH_EDITOR
EDataValidationResult FStateTreeMoveToTask::Compile(UE::StateTree::ICompileNodeContext& Context)
{
	const FInstanceDataType& InstanceData = Context.GetInstanceDataView().Get<FInstanceDataType>();

	// We only tick the task if we might track the destination vector
	if (Context.HasBindingForProperty(FName(TEXT("Destination"))))
	{
		if (InstanceData.bTrackMovingGoal || Context.HasBindingForProperty(FName(TEXT("bTrackMovingGoal"))))
		{
			if (!InstanceData.TargetActor && !Context.HasBindingForProperty(FName(TEXT("TargetActor"))))
			{
				// Flags on the base task are not saved. They will be set in Link().
				bSavedShouldCallTick = true;
			}
		}
	}

	return EDataValidationResult::Valid;
}

FText FStateTreeMoveToTask::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FText TargetValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, TargetActor)), Formatting);
	if (TargetValue.IsEmpty())
	{
		TargetValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, Destination)), Formatting);
	}

	if (Formatting == EStateTreeNodeFormatting::RichText)
	{
		return FText::Format(LOCTEXT("MoveToRich", "<b>Move To</> {0}"), TargetValue);	
	}
	return FText::Format(LOCTEXT("MoveTo", "Move To {0}"), TargetValue);
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE