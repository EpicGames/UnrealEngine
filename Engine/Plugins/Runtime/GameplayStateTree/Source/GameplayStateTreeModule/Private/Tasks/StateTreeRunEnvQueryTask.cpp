// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/StateTreeRunEnvQueryTask.h"

#include "EnvironmentQuery/EnvQuery.h"
#include "EnvironmentQuery/EnvQueryManager.h"
#include "StateTreeAsyncExecutionContext.h"
#include "StateTreeExecutionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StateTreeRunEnvQueryTask)

#define LOCTEXT_NAMESPACE "GameplayStateTree"

FStateTreeRunEnvQueryTask::FStateTreeRunEnvQueryTask()
{
	bShouldCallTick = false;
	bShouldCopyBoundPropertiesOnTick = false;
	bShouldCopyBoundPropertiesOnExitState = false;
}

EStateTreeRunStatus FStateTreeRunEnvQueryTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	if (!InstanceData.QueryTemplate)
	{
		return EStateTreeRunStatus::Failed;
	}

	FEnvQueryRequest Request(InstanceData.QueryTemplate, InstanceData.QueryOwner);

	for (FAIDynamicParam& DynamicParam : InstanceData.QueryConfig)
	{
		Request.SetDynamicParam(DynamicParam, nullptr);
	}

	InstanceData.RequestId = Request.Execute(InstanceData.RunMode,
		FQueryFinishedSignature::CreateLambda([WeakContext = Context.MakeWeakExecutionContext()](TSharedPtr<FEnvQueryResult> QueryResult) mutable
		{
			const FStateTreeStrongExecutionContext StrongContext = WeakContext.MakeStrongExecutionContext();
			if (FInstanceDataType* InstanceDataPtr = StrongContext.GetInstanceDataPtr<FInstanceDataType>())
			{
				InstanceDataPtr->RequestId = INDEX_NONE;

				bool bSuccess = false;
				if (QueryResult && QueryResult->IsSuccessful())
				{
					auto [VectorPtr, ActorPtr, ArrayOfVector, ArrayOfActor] = InstanceDataPtr->Result.GetPtrTupleFromStrongExecutionContext<FVector,AActor*, TArray<FVector>, TArray<AActor*>>(StrongContext);
					if (VectorPtr)
					{
						*VectorPtr = QueryResult->GetItemAsLocation(0);
					}
					else if (ActorPtr)
					{
						*ActorPtr = QueryResult->GetItemAsActor(0);
					}
					else if (ArrayOfVector)
					{
						QueryResult->GetAllAsLocations(*ArrayOfVector);
					}
					else if (ArrayOfActor)
					{
						QueryResult->GetAllAsActors(*ArrayOfActor);
					}

					bSuccess = true;
				}

				StrongContext.FinishTask(bSuccess ? EStateTreeFinishTaskType::Succeeded : EStateTreeFinishTaskType::Failed);
			}
		}));

	return InstanceData.RequestId != INDEX_NONE ? EStateTreeRunStatus::Running : EStateTreeRunStatus::Failed;
}

void FStateTreeRunEnvQueryTask::ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	if (InstanceData.RequestId != INDEX_NONE)
	{
		if (UEnvQueryManager* QueryManager = UEnvQueryManager::GetCurrent(Context.GetOwner()))
		{
			QueryManager->AbortQuery(InstanceData.RequestId);
		}
		InstanceData.RequestId = INDEX_NONE;
	}
}

#if WITH_EDITOR
void FStateTreeRunEnvQueryTask::PostEditInstanceDataChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent, FStateTreeDataView InstanceDataView)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FStateTreeRunEnvQueryInstanceData, QueryTemplate))
	{
		FInstanceDataType& InstanceData = InstanceDataView.GetMutable<FInstanceDataType>();
		if (InstanceData.QueryTemplate)
		{
			InstanceData.QueryTemplate->CollectQueryParams(*InstanceData.QueryTemplate, InstanceData.QueryConfig);
			for (FAIDynamicParam& DynamicParam : InstanceData.QueryConfig)
			{
				DynamicParam.bAllowBBKey = false;
			}
		}
		else
		{
			InstanceData.QueryConfig.Reset();
		}
	}
	else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FAIDynamicParam, bAllowBBKey))
	{
		FInstanceDataType& InstanceData = InstanceDataView.GetMutable<FInstanceDataType>();
		const int32 ChangedIndex = PropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_CHECKED(FStateTreeRunEnvQueryInstanceData, QueryConfig).ToString());
		if (InstanceData.QueryConfig.IsValidIndex(ChangedIndex))
		{
			if (!InstanceData.QueryConfig[ChangedIndex].bAllowBBKey)
			{
				InstanceData.QueryConfig[ChangedIndex].BBKey.InvalidateResolvedKey();
			}
		}
	}
}

FText FStateTreeRunEnvQueryTask::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	const FInstanceDataType* InstanceData = InstanceDataView.GetPtr<FInstanceDataType>();
	check(InstanceData);

	FText QueryTemplateValue = BindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(ID, GET_MEMBER_NAME_CHECKED(FInstanceDataType, QueryTemplate)), Formatting);
	if (QueryTemplateValue.IsEmpty())
	{
		QueryTemplateValue = FText::FromString(GetNameSafe(InstanceData->QueryTemplate));
	}

	if (Formatting == EStateTreeNodeFormatting::RichText)
	{
		return FText::Format(LOCTEXT("RunEQSRich", "<b>Run EQS Query</> {0}"), QueryTemplateValue);	
	}
	return FText::Format(LOCTEXT("RunEQS", "Run EQS Query {0}"), QueryTemplateValue);
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
