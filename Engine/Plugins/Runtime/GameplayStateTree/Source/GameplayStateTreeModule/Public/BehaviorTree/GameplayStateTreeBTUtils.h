// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTree/BehaviorTreeTypes.h"

enum class EStateTreeRunStatus : uint8;

namespace GameplayStateTreeBTUtils
{
	GAMEPLAYSTATETREEMODULE_API EBTNodeResult::Type StateTreeRunStatusToBTNodeResult(EStateTreeRunStatus Result);
}