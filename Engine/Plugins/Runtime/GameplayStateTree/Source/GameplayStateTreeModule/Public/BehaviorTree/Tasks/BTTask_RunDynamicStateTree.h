// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTree/BTTaskNode.h"
#include "StateTreeReference.h"
#include "StateTreeInstanceData.h"

#include "BTTask_RunDynamicStateTree.generated.h"

enum class EStateTreeRunStatus : uint8;

struct FStateTreeExecutionContext;

/**
 * RunDynamicStateTree task allows the execution of a state tree chosen at runtime.
 * UBTTask_RunDynamicStateTree::SetDynamicStateTree can be used to set the node's state tree.
 */
UCLASS(MinimalAPI)
class UBTTask_RunDynamicStateTree : public UBTTaskNode
{
	GENERATED_BODY()
public:
	DECLARE_DELEGATE_ThreeParams(FSetContextDataDelegate, FStateTreeExecutionContext&, UBehaviorTreeComponent&, FGameplayTag InjectionTag);

	GAMEPLAYSTATETREEMODULE_API static bool SetDynamicStateTree(UBehaviorTreeComponent& OwnerComp, FGameplayTag InInjectTag, const FStateTreeReference& InStateTree, const FSetContextDataDelegate& InSetContextDataDelegate, float InInterval, float InRandomDeviation, UBTCompositeNode* OptionalStartNode = nullptr);

	FGameplayTag GetInjectionTag() const { return InjectionTag; }

protected:
	GAMEPLAYSTATETREEMODULE_API UBTTask_RunDynamicStateTree(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	GAMEPLAYSTATETREEMODULE_API virtual FString GetStaticDescription() const override;
	GAMEPLAYSTATETREEMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;

	GAMEPLAYSTATETREEMODULE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	GAMEPLAYSTATETREEMODULE_API virtual void TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;
	GAMEPLAYSTATETREEMODULE_API virtual void OnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult) override;

	void SetStateTreeToRun(UBehaviorTreeComponent& OwnerComp, const FStateTreeReference& StateTreeToRun, const FSetContextDataDelegate& SetContextDelegate, float Interval, float RandomDeviation);

#if WITH_EDITOR
	GAMEPLAYSTATETREEMODULE_API const UObject* GetAssociatedAsset(TOptional<FBehaviorTreeNodeDebugContext> DebugContext) const override;
#endif //WITH_EDITOR

	UPROPERTY(Transient)
	FStateTreeReference StateTreeRef;

	UPROPERTY(Transient)
	FStateTreeInstanceData InstanceData;

	/** Gameplay tag that will identify this task for state tree injection */
	UPROPERTY(EditAnywhere, Category = Node)
	FGameplayTag InjectionTag;

	FSetContextDataDelegate SetContextDataDelegate;

	float Interval = 1.f;
	float RandomDeviation = 0.f;
};
