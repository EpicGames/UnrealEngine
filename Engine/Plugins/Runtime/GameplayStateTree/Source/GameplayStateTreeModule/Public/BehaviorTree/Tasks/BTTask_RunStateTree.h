// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTree/BTTaskNode.h"
#include "IStateTreeSchemaProvider.h"
#include "StateTreeReference.h"
#include "StateTreeInstanceData.h"

#include "BTTask_RunStateTree.generated.h"

struct FStateTreeExecutionContext;

/**
 * RunStateTree task allows the execution of state tree with the StateTreeAIComponentSchema inside a behavior tree.
 */
 UCLASS(MinimalAPI)
class UBTTask_RunStateTree : public UBTTaskNode, public IStateTreeSchemaProvider
{
	GENERATED_BODY()
public:
	GAMEPLAYSTATETREEMODULE_API UBTTask_RunStateTree(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:
	GAMEPLAYSTATETREEMODULE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	GAMEPLAYSTATETREEMODULE_API virtual void TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;
	GAMEPLAYSTATETREEMODULE_API virtual void OnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult) override;
	
	virtual bool SetContextRequirements(UBehaviorTreeComponent& OwnerComp, FStateTreeExecutionContext& Context);
	virtual bool CollectExternalData(const FStateTreeExecutionContext& Context, const UStateTree* StateTree, TArrayView<const FStateTreeExternalDataDesc> ExternalDataDescs, TArrayView<FStateTreeDataView> OutDataViews);

	virtual TSubclassOf<UStateTreeSchema> GetSchema() const override;

#if WITH_EDITOR
	GAMEPLAYSTATETREEMODULE_API virtual const UObject* GetAssociatedAsset(TOptional<FBehaviorTreeNodeDebugContext> DebugContext) const override;
#endif //WITH_EDITOR

	/** State tree that will be run when the task is selected. */
	UPROPERTY(EditAnywhere, Category = Task, DisplayName="State Tree", meta = (Schema = "/Script/GameplayStateTreeModule.StateTreeAIComponentSchema", SchemaCanBeOverriden))
	FStateTreeReference StateTreeRef;

	UPROPERTY(Transient)
	FStateTreeInstanceData InstanceData;

	/** Interval between state tree update. */
	UPROPERTY(EditAnywhere, Category = Task, meta = (ClampMin = "0.001"))
	float Interval = 0.01f;

	/** Random deviation on the interval between each state tree update. */
	UPROPERTY(EditAnywhere, Category = Task, meta = (ClampMin = "0.0"))
	float RandomDeviation = 0.0f;

	/** Saved actor used by the schema requirement. */
	TWeakObjectPtr<AActor> SchemaActor;
 };
