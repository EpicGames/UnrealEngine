// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/StateTreeComponent.h"

#include "StateTreeAIComponent.generated.h"

#define UE_API GAMEPLAYSTATETREEMODULE_API

/**
* State tree component designed to be run on an AIController.
* It uses the StateTreeAIComponentSchema that guarantees access to the AIController.
*/
UCLASS(MinimalAPI, ClassGroup = AI, Blueprintable, meta = (BlueprintSpawnableComponent))
class UStateTreeAIComponent : public UStateTreeComponent
{
	GENERATED_BODY()
public:
	//~ BEGIN IStateTreeSchemaProvider
	UE_API TSubclassOf<UStateTreeSchema> GetSchema() const override;
	//~ END
};

#undef UE_API
