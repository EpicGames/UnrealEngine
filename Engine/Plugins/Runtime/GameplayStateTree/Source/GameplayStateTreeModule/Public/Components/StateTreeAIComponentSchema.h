// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/StateTreeComponentSchema.h"

#include "StateTreeAIComponentSchema.generated.h"

#define UE_API GAMEPLAYSTATETREEMODULE_API

class AAIController;

/**
* State tree schema to be used with StateTreeAIComponent. 
* It guarantees access to an AIController and the Actor context value can be used to access the controlled pawn.
*/
UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories, meta = (DisplayName = "StateTree AI Component", CommonSchema))
class UStateTreeAIComponentSchema : public UStateTreeComponentSchema
{
	GENERATED_BODY()
public:
	UE_API UStateTreeAIComponentSchema(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	
	UE_API virtual void PostLoad() override;

	UE_API virtual bool IsStructAllowed(const UScriptStruct* InScriptStruct) const override;

	UE_API virtual void SetContextData(FContextDataSetter& ContextDataSetter, bool bLogErrors) const override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

protected:
	/** AIController class the StateTree is expected to run on. Allows to bind to specific Actor class' properties. */
	UPROPERTY(EditAnywhere, Category = "Defaults", NoClear)
	TSubclassOf<AAIController> AIControllerClass;
};

#undef UE_API
