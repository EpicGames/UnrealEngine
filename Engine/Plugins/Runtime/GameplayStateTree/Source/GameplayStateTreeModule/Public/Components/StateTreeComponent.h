// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BrainComponent.h"
#include "GameplayTaskOwnerInterface.h"
#include "IStateTreeSchemaProvider.h"
#include "StateTreeReference.h"
#include "StateTreeInstanceData.h"
#include "Templates/ValueOrError.h"
#include "UObject/Package.h"
#include "StateTreeComponent.generated.h"

#define UE_API GAMEPLAYSTATETREEMODULE_API

enum class EStateTreeRunStatus : uint8;
struct FGameplayTag;
struct FStateTreeEvent;
struct FStateTreeExecutionContext;

class UStateTree;
class UStateTreeComponent;

USTRUCT()
struct FStateTreeComponentExecutionExtension : public FStateTreeExecutionExtension
{
	GENERATED_BODY()

public:
	virtual void ScheduleNextTick(const FContextParameters& Context, const FNextTickArguments& Args) override;

	UPROPERTY()
	TObjectPtr<UStateTreeComponent> Component;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FStateTreeRunStatusChanged, EStateTreeRunStatus, StateTreeRunStatus);

UCLASS(MinimalAPI, Blueprintable, ClassGroup = AI, HideCategories = (Activation, Collision), meta = (BlueprintSpawnableComponent))
class UStateTreeComponent : public UBrainComponent, public IGameplayTaskOwnerInterface, public IStateTreeSchemaProvider
{
	GENERATED_BODY()
public:
	UE_API UStateTreeComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	//~ BEGIN UActorComponent overrides
	UE_API virtual void InitializeComponent() override;
	UE_API virtual void UninitializeComponent() override;
	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	//~ END UActorComponent overrides

	//~ BEGIN UBrainComponent overrides
	UE_API virtual void StartLogic() override;
	UE_API virtual void RestartLogic() override;
	UE_API virtual void StopLogic(const FString& Reason)  override;
	UE_API virtual void Cleanup() override;
	UE_API virtual void PauseLogic(const FString& Reason) override;
	UE_API virtual EAILogicResuming::Type ResumeLogic(const FString& Reason)  override;
	UE_API virtual bool IsRunning() const override;
	UE_API virtual bool IsPaused() const override;
	//~ END UBrainComponent overrides

	//~ BEGIN IGameplayTaskOwnerInterface
	UE_API virtual UGameplayTasksComponent* GetGameplayTasksComponent(const UGameplayTask& Task) const override;
	UE_API virtual AActor* GetGameplayTaskOwner(const UGameplayTask* Task) const override;
	UE_API virtual AActor* GetGameplayTaskAvatar(const UGameplayTask* Task) const override;
	UE_API virtual uint8 GetGameplayTaskDefaultPriority() const override;
	UE_API virtual void OnGameplayTaskInitialized(UGameplayTask& Task) override;
	//~ END IGameplayTaskOwnerInterface

	//~ BEGIN IStateTreeSchemaProvider
	UE_API virtual TSubclassOf<UStateTreeSchema> GetSchema() const override;
	//~ END

	/**
	 * Sets a new state tree.
	 * The state tree won't be set if the logic is running.
	 */
	UFUNCTION(BlueprintCallable, Category = "Gameplay|StateTree")
	UE_API void SetStateTree(UStateTree* StateTree);

	/**
	 * Sets a new state tree reference.
	 * The state tree reference won't be set if the logic is running.
	 */
	UFUNCTION(BlueprintCallable, Category = "Gameplay|StateTree")
	UE_API void SetStateTreeReference(FStateTreeReference StateTreeReference);

	/**
	 * Set the linked state tree overrides.
	 * The overrides won't be set if they do not use the StateTreeComponentSchema schema.
	 */
	UE_API void SetLinkedStateTreeOverrides(FStateTreeReferenceOverrides Overrides);

	/**
	 * Add a linked state tree override.
	 * The override won't be set if it doesn't use the StateTreeComponentSchema schema.
	 */
	UFUNCTION(BlueprintCallable, Category = "Gameplay|StateTree")
	UE_API void AddLinkedStateTreeOverrides(const FGameplayTag StateTag, FStateTreeReference StateTreeReference);

	/** Remove a linked state tree override. */
	UFUNCTION(BlueprintCallable, Category = "Gameplay|StateTree")
	UE_API void RemoveLinkedStateTreeOverrides(const FGameplayTag StateTag);
	
	/**
	 * Sets whether the State Tree is started automatically on begin play.
	 * This function sets the bStartLogicAutomatically property, and should be used mostly from constructions scripts.
	 * If you wish to start the logic manually, call StartLogic(). 
	 */
	UFUNCTION(BlueprintCallable, Category = "Gameplay|StateTree")
	UE_API void SetStartLogicAutomatically(const bool bInStartLogicAutomatically);
	 
	/** Sends event to the running StateTree. */
	UFUNCTION(BlueprintCallable, Category = "Gameplay|StateTree")
	UE_API void SendStateTreeEvent(const FStateTreeEvent& Event);

	/** Sends event to the running StateTree. */
	UE_API void SendStateTreeEvent(const FGameplayTag Tag, const FConstStructView Payload = FConstStructView(), const FName Origin = FName());

	/** Returns the current run status of the StateTree. */
	UFUNCTION(BlueprintPure, Category = "Gameplay|StateTree")
	UE_API EStateTreeRunStatus GetStateTreeRunStatus() const;

	/** Called when the run status of the StateTree has changed */
	UPROPERTY(BlueprintAssignable, Category = "Gameplay|StateTree")
	FStateTreeRunStatusChanged OnStateTreeRunStatusChanged;

#if WITH_GAMEPLAY_DEBUGGER
	UE_API virtual FString GetDebugInfoString() const override;

	/**
	 * @return the list of active states. 
	 * If the StateTree has linked asset StateTree, then more than one state can have the same name.
	 * Only used for debugging purposes.
	 */
	UE_API TArray<FName> GetActiveStateNames() const;
#endif // WITH_GAMEPLAY_DEBUGGER

protected:

#if WITH_EDITORONLY_DATA
	UE_API virtual void PostLoad() override;
#endif
	/**
	 * Called during initialize, will validate the state tree reference and create a context from the state tree to check its validity
	 * Override this function for custom state tree validation.
	 * Note: Override without calling super if the state tree reference is dynamically set after initialization
	 */
	UE_API virtual void ValidateStateTreeReference();

	/** @return a value if the state tree reference can be used by the component or the error why it's not a valid reference. */
	UE_API virtual TValueOrError<void, FString> HasValidStateTreeReference() const;

	UE_API virtual bool SetContextRequirements(FStateTreeExecutionContext& Context, bool bLogErrors = false);
	
	UE_API virtual bool CollectExternalData(const FStateTreeExecutionContext& Context, const UStateTree* StateTree, TArrayView<const FStateTreeExternalDataDesc> Descs, TArrayView<FStateTreeDataView> OutDataViews) const;

	UE_API void StartTree();
	UE_API void ScheduleTickFrame(const FStateTreeScheduledTick& NextTick);
	UE_API void ConditionalEnableTick();
	UE_API void DisableTick();
	
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "This property has been deprecated. Use StateTreeReference instead.")
	UPROPERTY()
	TObjectPtr<UStateTree> StateTree_DEPRECATED;
#endif

	/** State Tree asset to run on the component. */
	UPROPERTY(EditAnywhere, Category = AI, DisplayName = "State Tree", meta=(Schema="/Script/GameplayStateTreeModule.StateTreeComponentSchema", SchemaCanBeOverriden))
	FStateTreeReference StateTreeRef;

	/**
	 * Overrides for linked State Trees. This table is used to override State Tree references on linked states.
	 * If a linked state's tag is exact match of the tag specified on the table, the reference from the table is used instead.
	 */
	UPROPERTY(EditAnywhere, Category = AI, meta=(Schema="/Script/GameplayStateTreeModule.StateTreeComponentSchema"))
	FStateTreeReferenceOverrides LinkedStateTreeOverrides;

	UPROPERTY(Transient)
	FStateTreeInstanceData InstanceData;

	/** If true, the StateTree logic is started on begin play. Otherwise, StartLogic() needs to be called. */
	UPROPERTY(EditAnywhere, Category = AI)
	bool bStartLogicAutomatically = true;
	
	/** if set, execution has started and has not stopped yet. */
	uint8 bIsRunning : 1;
	
	/** if set, execution has been requested to stop ticking. */
	uint8 bIsPaused : 1;

private:
	FStateTreeExecutionContext* CurrentlyRunningExecContext = nullptr;
	
	friend FStateTreeComponentExecutionExtension;
};

#undef UE_API
