// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeSchema.h"
#include "GameFramework/Actor.h"
#include "StateTreeExecutionTypes.h"
#include "StateTreeComponentSchema.generated.h"

#define UE_API GAMEPLAYSTATETREEMODULE_API

class UBrainComponent;
class UStateTree;

struct FStateTreeExecutionContext;

UENUM()
enum class EStateTreeComponentSchemaScheduledTickPolicy : uint8
{
	Default,
	Allowed,
	Denied,
};

/**
 * StateTree for Actors with StateTree component. 
 */
UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories, meta = (DisplayName = "StateTree Component", CommonSchema))
class UStateTreeComponentSchema : public UStateTreeSchema
{
	GENERATED_BODY()

public:
	UE_API UStateTreeComponentSchema();

	UClass* GetContextActorClass() const { return ContextActorClass; };
	
	static UE_API bool SetContextRequirements(UBrainComponent& BrainComponent, FStateTreeExecutionContext& Context, bool bLogErrors = false);
	static UE_API bool CollectExternalData(const FStateTreeExecutionContext& Context, const UStateTree* StateTree, TArrayView<const FStateTreeExternalDataDesc> Descs, TArrayView<FStateTreeDataView> OutDataViews);

protected:
	UE_API virtual bool IsStructAllowed(const UScriptStruct* InScriptStruct) const override;
	UE_API virtual bool IsClassAllowed(const UClass* InScriptStruct) const override;
	UE_API virtual bool IsExternalItemAllowed(const UStruct& InStruct) const override;
	UE_API virtual bool IsScheduledTickAllowed() const override;

	/** Helper class to set the context data on the ExecutionContext */
	struct FContextDataSetter
	{
	public:
		UE_API FContextDataSetter(TNotNull<const UBrainComponent*> BrainComponent, FStateTreeExecutionContext& Context);

		TNotNull<const UBrainComponent*> GetComponent() const
		{
			return BrainComponent;
		}
		UE_API TNotNull<const UStateTree*> GetStateTree() const;
		UE_API TNotNull<const UStateTreeComponentSchema*> GetSchema() const;

		UE_API bool SetContextDataByName(FName Name, FStateTreeDataView DataView);

	private:
		TNotNull<const UBrainComponent*> BrainComponent;
		FStateTreeExecutionContext& ExecutionContext;
	};
	UE_API virtual void SetContextData(FContextDataSetter& ContextDataSetter, bool bLogErrors) const;

	UE_API virtual TConstArrayView<FStateTreeExternalDataDesc> GetContextDataDescs() const override;


	UE_API virtual void PostLoad() override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	
	const FStateTreeExternalDataDesc& GetContextActorDataDesc() const { return ContextDataDescs[0]; }
	FStateTreeExternalDataDesc& GetContextActorDataDesc() { return ContextDataDescs[0]; }

	/** Actor class the StateTree is expected to run on. Allows to bind to specific Actor class' properties. */
	UPROPERTY(EditAnywhere, Category="Defaults", NoClear)
	TSubclassOf<AActor> ContextActorClass;

	/**
	 * Indicates if the execution can sleep and the tick delayed.
	 * The default value set by the cvar StateTree.Component.DefaultScheduledTickAllowed
	 */
	UPROPERTY(EditAnywhere, Category="Defaults")
	EStateTreeComponentSchemaScheduledTickPolicy ScheduledTickPolicy = EStateTreeComponentSchemaScheduledTickPolicy::Default;
	
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.4, "ContextActorDataDesc is being replaced with ContextDataDescs. Call GetContextActorDataDesc to access the equivalent.")
	UPROPERTY()
	FStateTreeExternalDataDesc ContextActorDataDesc_DEPRECATED;
#endif

	UPROPERTY()
	TArray<FStateTreeExternalDataDesc> ContextDataDescs;
};

#undef UE_API
