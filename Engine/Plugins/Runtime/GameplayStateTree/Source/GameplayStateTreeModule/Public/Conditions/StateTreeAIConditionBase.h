// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeConditionBase.h"

#include "StateTreeAIConditionBase.generated.h"

// Base class of all AI condition that expect to be run on an AIController or derived class 
USTRUCT(meta = (Hidden, Category = "AI"))
struct FStateTreeAIConditionBase : public FStateTreeConditionBase
{
	GENERATED_BODY()
};