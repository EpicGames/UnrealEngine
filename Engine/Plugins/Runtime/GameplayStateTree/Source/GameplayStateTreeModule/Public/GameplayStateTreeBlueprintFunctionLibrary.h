// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "GameplayStateTreeBlueprintFunctionLibrary.generated.h"

#define UE_API GAMEPLAYSTATETREEMODULE_API

class UStateTree;
class AActor;

UCLASS(MinimalAPI, meta = (ScriptName = "GameplayStateTreeLibrary"))
class UGameplayStateTreeBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	
	/**
	 * Starts executing an instance of the provided state tree asset on the specified actor.
	 * This will use the existing required component if present or add it if missing.
	 * @param Actor The actor on which the StateTree must be executed.
	 * @param StateTreeAsset The state tree asset to execute
	 * @return Whether a state tree instance was successfully started
	 */
	UFUNCTION(BlueprintCallable, Category = "AI", meta=(ReturnDisplayName="bSuccess"))
	static UE_API bool RunStateTree(AActor* Actor, UStateTree* StateTreeAsset);
};

#undef UE_API
