// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeTaskBase.h"

#include "StateTreeAITask.generated.h"

// Base class of all AI task that expect to be run on an AIController or derived class
USTRUCT(meta= (Hidden, Category = "AI"))
struct FStateTreeAITaskBase : public FStateTreeTaskBase
{
	GENERATED_BODY()
};

// Base class of all AI task that do a physical action 
USTRUCT(meta = (Hidden, Category="AI|Action"))
struct FStateTreeAIActionTaskBase : public FStateTreeAITaskBase
{
	GENERATED_BODY()
};