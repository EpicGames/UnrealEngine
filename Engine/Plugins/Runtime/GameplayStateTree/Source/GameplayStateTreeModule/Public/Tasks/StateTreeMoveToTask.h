// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AISystem.h"
#include "NavFilters/NavigationQueryFilter.h"
#include "Tasks/StateTreeAITask.h"
#include "Templates/SubclassOf.h"

#include "StateTreeMoveToTask.generated.h"

#define UE_API GAMEPLAYSTATETREEMODULE_API

class AActor;
class AAIController;
class IGameplayTaskOwnerInterface;
class UAITask_MoveTo;

struct FAIMoveRequest;

USTRUCT()
struct FStateTreeMoveToTaskInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Context)
	TObjectPtr<AAIController> AIController = nullptr;

	UPROPERTY(EditAnywhere, Category = Parameter)
	FVector Destination = FVector::Zero();

	UPROPERTY(EditAnywhere, Category = Parameter)
	TObjectPtr<AActor> TargetActor;

	/** fixed distance added to threshold between AI and goal location in destination reach test */
	UPROPERTY(EditAnywhere, Category = Parameter, meta=(ClampMin = "0.0", UIMin="0.0"))
	float AcceptableRadius = GET_AI_CONFIG_VAR(AcceptanceRadius);

	/** if the task is expected to react to changes to location in input
	 *	this property can be used to tweak sensitivity of the mechanism. Value is 
	 *	recommended to be less than AcceptableRadius */
	UPROPERTY(EditAnywhere, Category = Parameter, meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition="bTrackMovingGoal"))
	float DestinationMoveTolerance = 0.f;

	/** "None" will result in default filter being used */
	UPROPERTY(EditAnywhere, Category = Parameter)
	TSubclassOf<UNavigationQueryFilter> FilterClass;

	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bAllowStrafe = GET_AI_CONFIG_VAR(bAllowStrafing);

	/** if set, use incomplete path when goal can't be reached */
	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bAllowPartialPath = GET_AI_CONFIG_VAR(bAcceptPartialPaths);

	/** if set, path to goal actor will update itself when actor moves */
	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bTrackMovingGoal = true;

	/** if set, the goal location will need to be navigable */
	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bRequireNavigableEndLocation = true;

	/** if set, goal location will be projected on navigation data (navmesh) before using */
	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bProjectGoalLocation = true;

	/** if set, radius of AI's capsule will be added to threshold between AI and goal location in destination reach test  */
	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bReachTestIncludesAgentRadius = GET_AI_CONFIG_VAR(bFinishMoveOnGoalOverlap);
	
	/** if set, radius of goal's capsule will be added to threshold between AI and goal location in destination reach test  */
	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bReachTestIncludesGoalRadius = GET_AI_CONFIG_VAR(bFinishMoveOnGoalOverlap);

	UPROPERTY(Transient)
	TObjectPtr<UAITask_MoveTo> MoveToTask = nullptr;

	UPROPERTY(Transient)
	TScriptInterface<IGameplayTaskOwnerInterface> TaskOwner = nullptr;
};


/**
* Task moving the given AIController's pawn using the AITask_MoveTo framework. Succeed when move ends at destinations, fails if move is impossible.
*/
USTRUCT(meta = (DisplayName = "Move To", Category = "AI|Action"))
struct FStateTreeMoveToTask : public FStateTreeAIActionTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeMoveToTaskInstanceData;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API FStateTreeMoveToTask();
	UE_API virtual bool Link(FStateTreeLinker& Linker) override;
	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;
	UE_API virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;

	UE_API virtual UAITask_MoveTo* PrepareMoveToTask(FStateTreeExecutionContext& Context, AAIController& Controller, UAITask_MoveTo* ExistingTask, FAIMoveRequest& MoveRequest) const;
	UE_API virtual EStateTreeRunStatus PerformMoveTask(FStateTreeExecutionContext& Context, AAIController& Controller) const;

#if WITH_EDITOR
	UE_API virtual EDataValidationResult Compile(UE::StateTree::ICompileNodeContext& Context) override;

	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	virtual FName GetIconName() const override
	{
		return FName("StateTreeEditorStyle|Node.Movement");
	}
	virtual FColor GetIconColor() const override
	{
		return UE::StateTree::Colors::Grey;
	}
#endif // WITH_EDITOR

	UPROPERTY()
	bool bSavedShouldCallTick = false;
};

#undef UE_API