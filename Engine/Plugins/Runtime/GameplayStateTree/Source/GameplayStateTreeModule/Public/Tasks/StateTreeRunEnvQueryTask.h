// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeTaskBase.h"
#include "StateTreePropertyRef.h"
#include "EnvironmentQuery/EnvQueryTypes.h"

#include "StateTreeRunEnvQueryTask.generated.h"

#define UE_API GAMEPLAYSTATETREEMODULE_API

USTRUCT()
struct FStateTreeRunEnvQueryInstanceData
{
	GENERATED_BODY()

	// Result of the query. If an array is binded, it will output all the created values otherwise it will output the best one.
	UPROPERTY(EditAnywhere, Category = Out, meta = (RefType = "/Script/CoreUObject.Vector, /Script/Engine.Actor", CanRefToArray))
	FStateTreePropertyRef Result;

	// The query will be run with this actor has the owner object.
	UPROPERTY(EditAnywhere, Category = Context)
	TObjectPtr<AActor> QueryOwner = nullptr;

	// The query template to run
	UPROPERTY(EditAnywhere, Category = Parameter)
	TObjectPtr<UEnvQuery> QueryTemplate;

	// Query config associated with the query template.
	UPROPERTY(EditAnywhere, EditFixedSize, Category = Parameter)
	TArray<FAIDynamicParam> QueryConfig;

	/** determines which item will be stored (All = only first matching) */
	UPROPERTY(EditAnywhere, Category = Parameter)
	TEnumAsByte<EEnvQueryRunMode::Type> RunMode = EEnvQueryRunMode::SingleResult;

	int32 RequestId = INDEX_NONE;
};

/**
* Task that runs an async environment query and outputs the result to an outside parameter. Supports Actor and vector types EQS.
* The task is usually run in a sibling state to the result user will be with the data being stored in the parent state's parameters.
* - Parent (Has an EQS result parameter)
*	- Run Env Query (If success go to Use Query Result)
*	- Use Query Result
*/
USTRUCT(meta = (DisplayName = "Run Env Query", Category = "Common"))
struct FStateTreeRunEnvQueryTask : public FStateTreeTaskCommonBase
{
	GENERATED_BODY()

	using FInstanceDataType = FStateTreeRunEnvQueryInstanceData;

	UE_API FStateTreeRunEnvQueryTask();

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;

#if WITH_EDITOR
	UE_API virtual void PostEditInstanceDataChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent, FStateTreeDataView InstanceDataView) override;
	UE_API virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	virtual FName GetIconName() const override
	{
		return FName("StateTreeEditorStyle|Node.Find");
	}
	virtual FColor GetIconColor() const override
	{
		return UE::StateTree::Colors::Grey;
	}
#endif // WITH_EDITOR
};

#undef UE_API